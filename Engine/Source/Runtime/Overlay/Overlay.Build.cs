// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class Overlay : ModuleRules
	{
		public Overlay(ReadOnlyTargetRules Target) : base(Target)
		{
            PublicDependencyModuleNames.AddRange(
                new string[] {
				    "Core",
                    "CoreUObject",
                    "Engine"
                }
            );

            PrivateDependencyModuleNames.AddRange(
                new string[] {
                    "Slate",
                    "SlateCore",
                    "UMG",
                }
            );
        }
	}
}