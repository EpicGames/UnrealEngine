// Copyright Epic Games, Inc. All Rights Reserved.

#include "BasicOverlays.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BasicOverlays)

#include "UObject/AssetRegistryTagsContext.h"
#if WITH_EDITORONLY_DATA
#include "EditorFramework/AssetImportData.h"
#endif

void UBasicOverlays::PostInitProperties()
{
#if WITH_EDITORONLY_DATA
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		AssetImportData = NewObject<UAssetImportData>(this, TEXT("AssetImportData"));
	}
#endif

	Super::PostInitProperties();
}

void UBasicOverlays::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UBasicOverlays::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
#if WITH_EDITORONLY_DATA
	if (AssetImportData)
	{
		Context.AddTag(FAssetRegistryTag(SourceFileTagName(), AssetImportData->GetSourceData().ToJson(), FAssetRegistryTag::TT_Hidden));
	}
#endif	// WITH_EDITORONLY_DATA

	Super::GetAssetRegistryTags(Context);
}

TArray<FOverlayItem> UBasicOverlays::GetAllOverlays() const
{
	return Overlays;
}

void UBasicOverlays::GetOverlaysForTime(const FTimespan& Time, TArray<FOverlayItem>& OutOverlays) const
{
	OutOverlays.Empty();

	for (const FOverlayItem& Overlay : Overlays)
	{
		if (Overlay.StartTime <= Time && Time < Overlay.EndTime)
		{
			OutOverlays.Add(Overlay);
		}
	}
}
