// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalizedOverlays.h"

#include "BasicOverlays.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Culture.h"
#include "UObject/AssetRegistryTagsContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LocalizedOverlays)

#if WITH_EDITORONLY_DATA
#include "EditorFramework/AssetImportData.h"
#endif	// WITH_EDITORONLY_DATA

void ULocalizedOverlays::PostInitProperties()
{
#if WITH_EDITORONLY_DATA
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		AssetImportData = NewObject<UAssetImportData>(this, TEXT("AssetImportData"));
	}
#endif

	Super::PostInitProperties();
}

void ULocalizedOverlays::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void ULocalizedOverlays::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
#if WITH_EDITORONLY_DATA
	if (AssetImportData)
	{
		Context.AddTag(FAssetRegistryTag(SourceFileTagName(), AssetImportData->GetSourceData().ToJson(), FAssetRegistryTag::TT_Hidden));
	}
#endif	// WITH_EDITORONLY_DATA

	Super::GetAssetRegistryTags(Context);
}

UBasicOverlays* ULocalizedOverlays::GetCurrentLocaleOverlays() const
{
	UBasicOverlays* OverlaysToUse = DefaultOverlays;

	// Determine what out current culture is, and grab the most appropriate set of subtitles for it
	FInternationalization& Internationalization = FInternationalization::Get();

	const TArray<FString> PrioritizedCultureNames = Internationalization.GetPrioritizedCultureNames(Internationalization.GetCurrentCulture()->GetName());

	for (const FString& CultureName : PrioritizedCultureNames)
	{
		if (LocaleToOverlaysMap.Contains(CultureName))
		{
			OverlaysToUse = *LocaleToOverlaysMap.Find(CultureName);
			break;
		}
	}

	return OverlaysToUse;
}

TArray<FOverlayItem> ULocalizedOverlays::GetAllOverlays() const
{
	UBasicOverlays* OverlaysToUse = GetCurrentLocaleOverlays();

	return (OverlaysToUse != nullptr) ? OverlaysToUse->GetAllOverlays() : TArray<FOverlayItem>();
}

void ULocalizedOverlays::GetOverlaysForTime(const FTimespan& Time, TArray<FOverlayItem>& OutOverlays) const
{
	OutOverlays.Empty();

	UBasicOverlays* OverlaysToUse = GetCurrentLocaleOverlays();

	if (OverlaysToUse != nullptr)
	{
		OverlaysToUse->GetOverlaysForTime(Time, OutOverlays);
	}
}
