// Copyright Epic Games, Inc. All Rights Reserved.

#include "OverlaysImporter.h"

#include "Misc/FileHelper.h"
#include "Misc/Paths.h"

FOverlaysImporter::FOverlaysImporter()
	: FileType(EOverlaysFileType::Unknown)
{
}

FOverlaysImporter::~FOverlaysImporter()
{
}

bool FOverlaysImporter::OpenFile(const FString& FilePath)
{
	Reset();
	bool bValidFile = false;
	Filename = FilePath;

	// Check to see if this is a SubRip Subtitles file
	if (FPaths::GetExtension(Filename) == TEXT("srt"))
	{
		FFileHelper::LoadFileToString(FileContents, *Filename);

		// Trim off leading whitespace
		FileContents.TrimStartInline();

		// SubRip Subtitle files should begin with the line "1". If it doesn't, it's invalid or not a SubRip file
		int32 IndexOfNewLine = INDEX_NONE; // There should also be at least two lines in the file.
		FString FirstLine = FileContents.Left(FileContents.FindChar(TEXT('\n'), IndexOfNewLine)).TrimStartAndEnd();

		bValidFile = IndexOfNewLine != INDEX_NONE && FirstLine == TEXT("1");

		if (bValidFile)
		{
			FileType = EOverlaysFileType::SubRipSubtitles;
		}
	}

	if (!bValidFile)
	{
		// If the file isn't valid, dump the file contents and ensure that we don't try to import anything
		Reset();
	}

	return bValidFile;
}

void FOverlaysImporter::Reset()
{
	Filename.Empty();
	FileContents.Empty();
	FileType = EOverlaysFileType::Unknown;
}

bool FOverlaysImporter::ImportBasic(TArray<FOverlayItem>& OutOverlays) const
{
	bool bImported = false;

	OutOverlays.Empty();

	switch (FileType)
	{
		case EOverlaysFileType::SubRipSubtitles:
		{
			bImported = ParseSubRipSubtitles(OutOverlays);
		}
		break;

		case EOverlaysFileType::Unknown:
		default:
		{
			bImported = false;
		}
		break;
	}

	return bImported;
}

bool FOverlaysImporter::ParseSubRipSubtitles(TArray<FOverlayItem>& OutSubtitles) const
{
	// Parse the file content into separate lines.
	TArray<FString> Lines;
	FileContents.ParseIntoArrayLines(Lines, false);	// Do not cull out empty strings, since they determine when the next subtitle occurs

	int32 LineIndex = 0;

	while (LineIndex < Lines.Num())
	{
		// Subtitle blocks are numbered sequentially, starting from 1.
		FString ExpectedSubtitleIndex = FString::FromInt(OutSubtitles.Num() + 1);
		Lines[LineIndex].TrimStartAndEndInline();
		FString TrimmedIndexLine = Lines[LineIndex++];

		if (ExpectedSubtitleIndex != TrimmedIndexLine)
		{
			// We're not where we think we are, or the file is malformed.
			ensureMsgf(false, TEXT("Failed to parse %s - subtitle indices are not sequential or the file is malformed."), *Filename);
			break;
		}

		FOverlayItem CurrentSubtitle;

		// The next line is a timestamp line, in the format "start --> end"
		// Each timestamp is written and padded out in the format 00:00:00,000 - and it uses a comma instead of a decimal as the fractional separator
		TArray<FString> Timespans;
		Lines[LineIndex++].ParseIntoArray(Timespans, TEXT("-->"));

		if (Timespans.Num() != 2)
		{
			ensureMsgf(false, TEXT("Failed to parse %s - subtitle %s does not have a defined start and end time"), *Filename, *ExpectedSubtitleIndex);
			break;
		}

		// sanitize any whitespace from parsing
		Timespans[0].TrimStartAndEndInline();
		Timespans[1].TrimStartAndEndInline();

		if (!FTimespan::Parse(Timespans[0], CurrentSubtitle.StartTime) || !FTimespan::Parse(Timespans[1], CurrentSubtitle.EndTime))
		{
			ensureMsgf(false, TEXT("Failed to parse %s - times are malformed for subtitle %s"), *Filename, *ExpectedSubtitleIndex);
			break;
		}

		// Until we reach an empty line (or the end of the file), whatever text we find will be used for the actual subtitle.
		while (LineIndex < Lines.Num())
		{
			Lines[LineIndex].TrimStartInline();
			if(Lines[LineIndex].IsEmpty())
			{
				break;
			}

			if (!CurrentSubtitle.Text.IsEmpty())
			{
				CurrentSubtitle.Text.Append(TEXT("\n"));
			}

			CurrentSubtitle.Text.Append(Lines[LineIndex++]);
		}

		// Add the created subtitle
		OutSubtitles.Add(CurrentSubtitle);

		// Advance to the next subtitle block, skipping any blank lines along the way
		while (LineIndex < Lines.Num())
		{
			Lines[LineIndex].TrimStartInline();
			if(!Lines[LineIndex].IsEmpty())
			{
				break;
			}
			++LineIndex;
		}
	}

	return LineIndex == Lines.Num();
}