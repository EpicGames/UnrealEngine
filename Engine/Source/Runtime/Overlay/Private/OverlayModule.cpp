// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "OverlayPrivate.h"
#include "IOverlayModule.h"

DEFINE_LOG_CATEGORY(LogOverlay);

/**
 * Implements the Overlay module
 */
class FOverlayModule
	: public IOverlayModule
{
public:

	// IModuleInterface interface

	virtual void StartupModule() override 
	{
#if WITH_EDITOR
		// Ensure that we always have the overlays factories when loading this module in an editor build
		FModuleManager::Get().LoadModule(TEXT("OverlayEditor"));
#endif	// WITH_EDITOR
	}

	virtual void ShutdownModule() override { }

	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

	// End IModuleInterface interface
};

IMPLEMENT_MODULE(FOverlayModule, Overlay);