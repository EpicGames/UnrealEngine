// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "Misc/Timespan.h"
#include "Overlays.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ScriptMacros.h"
#include "UObject/UObjectGlobals.h"

#include "BasicOverlays.generated.h"

class UAssetImportData;
class UObject;

/**
 * Implements an asset that contains a set of overlay data (which includes timing, text, and position) to be displayed for any
 * given source (including, but not limited to, audio, dialog, and movies)
 */
UCLASS(BlueprintType, hidecategories = (Object), MinimalAPI)
class UBasicOverlays
	: public UOverlays
{
	GENERATED_BODY()

public:

	/** The overlay data held by this asset. Contains info on timing, position, and the subtitle to display */
	UPROPERTY(EditAnywhere, Category="Overlay Data")
	TArray<FOverlayItem> Overlays;

#if WITH_EDITORONLY_DATA

	/** The import data used to make this overlays asset */
	UPROPERTY(VisibleAnywhere, Instanced, Category="Import Settings")
	TObjectPtr<UAssetImportData> AssetImportData;

#endif	// WITH_EDITORONLY_DATA

public:

	//~ UOverlays interface

	OVERLAY_API virtual TArray<FOverlayItem> GetAllOverlays() const override;
	OVERLAY_API virtual void GetOverlaysForTime(const FTimespan& Time, TArray<FOverlayItem>& OutOverlays) const override;

public:
	
	//~ UObject interface
	
	OVERLAY_API virtual void PostInitProperties() override;
	OVERLAY_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	OVERLAY_API virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;
	// End UObject interface

private:

};
