// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

/**
 * Interface for the Overlay module
 */
class IOverlayModule
	: public IModuleInterface
{
public:
	/** Virtual destructor */
	virtual ~IOverlayModule() = default;
};
