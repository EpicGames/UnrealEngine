// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Misc/Timespan.h"
#include "Overlays.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ScriptMacros.h"
#include "UObject/UObjectGlobals.h"

#include "LocalizedOverlays.generated.h"

class FString;
class UAssetImportData;
class UBasicOverlays;
class UObject;

/**
 * Implements an asset that contains a set of Basic Overlays that will be displayed in accordance with
 * the current locale, or a default set if an appropriate locale is not found
 */
UCLASS(BlueprintType, hidecategories = (Object), MinimalAPI)
class ULocalizedOverlays
	: public UOverlays
{
	GENERATED_BODY()

public:

	/** The overlays to use if no overlays are found for the current culture */
	UPROPERTY(EditAnywhere, Category="Overlay Data")
	TObjectPtr<UBasicOverlays> DefaultOverlays;

	/**
	 * Maps a set of cultures to specific BasicOverlays assets.
	 * Cultures are comprised of three hyphen-separated parts:
	 *		A two-letter ISO 639-1 language code (e.g., "zh")
	 *		An optional four-letter ISO 15924 script code (e.g., "Hans")
	 *		An optional two-letter ISO 3166-1 country code  (e.g., "CN")
	 */
	UPROPERTY(EditAnywhere, Category="Overlay Data")
	TMap<FString, TObjectPtr<UBasicOverlays>> LocaleToOverlaysMap;

#if WITH_EDITORONLY_DATA

	/** The import data used to make this overlays asset */
	UPROPERTY(VisibleAnywhere, Instanced, Category="Import Settings")
	TObjectPtr<UAssetImportData> AssetImportData;

#endif	// WITH_EDITORONLY_DATA

public:

	//~ UOverlays interface

	OVERLAY_API virtual TArray<FOverlayItem> GetAllOverlays() const override;
	OVERLAY_API virtual void GetOverlaysForTime(const FTimespan& Time, TArray<FOverlayItem>& OutOverlays) const override;


public:

	//~ UObject interface

	OVERLAY_API virtual void PostInitProperties() override;
	OVERLAY_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	OVERLAY_API virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;

private:
	
	/**
	 * Retrieves the overlays object for the current locale
	 */
	UBasicOverlays* GetCurrentLocaleOverlays() const;
};
