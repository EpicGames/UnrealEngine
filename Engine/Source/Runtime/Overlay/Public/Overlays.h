// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Math/Vector2D.h"
#include "Misc/Timespan.h"
#include "Overlays.generated.h"


USTRUCT(BlueprintType)
struct FOverlayItem
{
	GENERATED_BODY()

	FOverlayItem()
		: StartTime(0)
		, EndTime(0)
		, Position(0.0f, 0.0f)
	{
	}

	/** When the overlay should be displayed */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Display Times")
	FTimespan StartTime;

	/** When the overlay should be cleared */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Display Times")
	FTimespan EndTime;

	/** Text that appears onscreen when the overlay is shown */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Display Data", meta=(MultiLine=True))
	FString Text;

	/** The position of the text on screen (Between 0.0 and 1.0) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Display Data")
	FVector2D Position;
};


/**
 * An interface class for creating overlay data assets
 */
UCLASS(Abstract, MinimalAPI)
class UOverlays
	: public UObject
{
	GENERATED_BODY()

public:

	/**
	 * Retrieves the set of overlays that should be used by this object (must be implemented by child classes)
	 */
	virtual TArray<FOverlayItem> GetAllOverlays() const PURE_VIRTUAL(UOverlays::GetAllOverlays, return TArray<FOverlayItem>(););

	/**
	 * Retrieves the set of overlays associated with this object for the given timespan (must be implemented by child classes)
	 * 
	 * @param	Time			Determines what overlays should be displayed
	 * @param	OutOverlays		All overlays that should be displayed for the given Time. Expect this to be emptied.
	 */
	virtual void GetOverlaysForTime(const FTimespan& Time, TArray<FOverlayItem>& OutOverlays) const PURE_VIRTUAL(UOverlays::GetAllOverlays, return;);
};
