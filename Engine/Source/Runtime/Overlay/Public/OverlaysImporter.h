// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Overlays.h"

struct FOverlayItem;

enum class EOverlaysFileType
{
	Unknown,
	SubRipSubtitles,
};

class FOverlaysImporter
{
public:
	OVERLAY_API FOverlaysImporter();
	OVERLAY_API ~FOverlaysImporter();

	/**
	 * Opens the file and preparses it for import
	 *
	 * @param	Filename	The file to open
	 * @return	True if the file was opened successfully, false if the file could not be opened or is not an overlay file.
	 */
	OVERLAY_API bool OpenFile(const FString& FilePath);

	/**
	 * Parses the supplied import file for basic overlay data
	 *
	 * @param	OutSubtitles	The output array where new overlays are stored. This is emptied when the import begins
	 * @return	True if the file was parsed successfully
	 */
	OVERLAY_API bool ImportBasic(TArray<FOverlayItem>& OutOverlays) const;

	/**
	 * Resets the importer to a default state
	 */
	OVERLAY_API void Reset();

private:
	bool ParseSubRipSubtitles(TArray<FOverlayItem>& OutSubtitles) const;

private:
	FString Filename;
	FString FileContents;

	EOverlaysFileType FileType;
};
